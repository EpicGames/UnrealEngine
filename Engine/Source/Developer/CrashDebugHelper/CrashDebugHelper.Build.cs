// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CrashDebugHelper : ModuleRules
{
	public CrashDebugHelper(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateIncludePaths.AddRange(
			new string[] {
				"Developer/CrashDebugHelper/Private/",
				"Developer/CrashDebugHelper/Private/Linux",
				"Developer/CrashDebugHelper/Private/Mac",
				"Developer/CrashDebugHelper/Private/Windows",
				"Developer/CrashDebugHelper/Private/IOS",
			}
		);

		if (Target.Type != TargetType.Game && Target.Type != TargetType.Client)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				}
			);
		}
		else
		{
			IsRedistributableOverride = true;
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				}
			);
		}

		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.WindowsPlatform.bUseBundledDbgHelp)
		{
			throw new System.Exception("CrashDebugHelper uses DBGENG.DLL at runtime, which depends on a matching version of DBGHELP.DLL but cannot be redistributed. Please set WindowsPlatform.bUseBundledDbgHelp = false for this target.");
		}

		// Note: PLCrashReporter is not supported on tvOS.
		PrivateIncludePathModuleNames.Add("PLCrashReporter");
		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.IOS)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, new string[]
			{
				"PLCrashReporter"
			});
		}
	}
}
