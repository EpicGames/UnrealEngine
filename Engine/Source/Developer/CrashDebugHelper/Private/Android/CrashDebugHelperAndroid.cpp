// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDebugHelperAndroid.h"

bool FCrashDebugHelperAndroid::CreateMinidumpDiagnosticReport( const FString& InCrashDumpName )
{
	return true;
}
