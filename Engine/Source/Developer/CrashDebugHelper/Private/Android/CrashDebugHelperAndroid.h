// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CrashDebugHelper.h"

class FString;

class FCrashDebugHelperAndroid : public ICrashDebugHelper
{
public:
	/**
	 *	Parse the given crash dump, and generate a report. 
	 *
	 *	@param	InCrashDumpName		The crash dump file to process
	 *
	 *	@return	bool				true if successful, false if not
	 */
	virtual bool CreateMinidumpDiagnosticReport( const FString& InCrashDumpName ) override;
};

typedef FCrashDebugHelperAndroid FCrashDebugHelper;
