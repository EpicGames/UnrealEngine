// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDebugHelper.h"
#include "HAL/FileManager.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Stats/StatsMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "CrashDebugHelperPrivate.h"
#include "Misc/EngineVersion.h"

/*-----------------------------------------------------------------------------
	ICrashDebugHelper
-----------------------------------------------------------------------------*/

bool ICrashDebugHelper::Init()
{
	bInitialized = true;

	// Check if we have a valid EngineVersion, if so use it.
	FString CmdEngineVersion;
	const bool bHasEngineVersion = FParse::Value( FCommandLine::Get(), TEXT( "EngineVersion=" ), CmdEngineVersion );
	if( bHasEngineVersion )
	{
		FEngineVersion EngineVersion;
		FEngineVersion::Parse( CmdEngineVersion, EngineVersion );

		// Clean branch name.
		CrashInfo.DepotName = EngineVersion.GetBranch();
		CrashInfo.BuiltFromCL = (int32)EngineVersion.GetChangelist();

		CrashInfo.EngineVersion = CmdEngineVersion;
	}
	else
	{
		// Use the current values.
		const FEngineVersion& EngineVersion = FEngineVersion::Current();
		CrashInfo.DepotName = EngineVersion.GetBranch();
		CrashInfo.BuiltFromCL = (int32)EngineVersion.GetChangelist();
		CrashInfo.EngineVersion = EngineVersion.ToString();
	}

	// Check if we have a valid BuildVersion, if so use it.
	FString CmdBuildVersion;
	const bool bHasBuildVersion = FParse::Value(FCommandLine::Get(), TEXT("BuildVersion="), CmdBuildVersion);
	if (bHasBuildVersion)
	{
		CrashInfo.BuildVersion = CmdBuildVersion;
	}
	else
	{
		CrashInfo.BuildVersion = FApp::GetBuildVersion();
	}

	FString PlatformName;
	const bool bHasPlatformName = FParse::Value(FCommandLine::Get(), TEXT("PlatformName="), PlatformName);
	if (bHasPlatformName)
	{
		CrashInfo.PlatformName = PlatformName;
	}
	else
	{
		// Use the current values.
		CrashInfo.PlatformName = FPlatformProperties::PlatformName();
	}

	FString PlatformVariantName;
	const bool bHasPlatformVariantName = FParse::Value(FCommandLine::Get(), TEXT("PlatformVariantName="), PlatformVariantName);
	if (bHasPlatformVariantName)
	{
		CrashInfo.PlatformVariantName = PlatformVariantName;
	}
	else
	{
		// Use the basic platform name.
		CrashInfo.PlatformVariantName = CrashInfo.PlatformName;
	}

	UE_LOGF( LogCrashDebugHelper, Log, "DepotName: %ls", *CrashInfo.DepotName );
	UE_LOGF( LogCrashDebugHelper, Log, "BuiltFromCL: %i", CrashInfo.BuiltFromCL );
	UE_LOGF( LogCrashDebugHelper, Log, "EngineVersion: %ls", *CrashInfo.EngineVersion );
	UE_LOGF( LogCrashDebugHelper, Log, "BuildVersion: %ls", *CrashInfo.BuildVersion );

	return bInitialized;
}

bool ICrashDebugHelper::ReadSourceFile( TArray<FString>& OutStrings )
{
	const FString FilePath = FPaths::RootDir() / CrashInfo.SourceFile;

	FString Line;
	if (FFileHelper::LoadFileToString( Line, *FilePath ))
	{
		Line = Line.Replace( TEXT( "\r" ), TEXT( "" ) );
		Line.ParseIntoArray( OutStrings, TEXT( "\n" ), false );
		UE_LOGF( LogCrashDebugHelper, Log, "Reading a single source file: %ls", *FilePath );
		return true;
	}
	else
	{
		UE_LOGF( LogCrashDebugHelper, Warning, "Failed to open source file: %ls", *FilePath );
		return false;
	}
}

void ICrashDebugHelper::AddSourceToReport()
{
	if( CrashInfo.SourceFile.Len() > 0 && CrashInfo.SourceLineNumber != 0 )
	{
		TArray<FString> Lines;
		ReadSourceFile( Lines );

		const uint32 MinLine = FMath::Clamp( CrashInfo.SourceLineNumber - 15, (uint32)1, (uint32)Lines.Num() );
		const uint32 MaxLine = FMath::Clamp( CrashInfo.SourceLineNumber + 15, (uint32)1, (uint32)Lines.Num() );

		for( uint32 Line = MinLine; Line < MaxLine; Line++ )
		{
			if( Line == CrashInfo.SourceLineNumber - 1 )
			{
				CrashInfo.SourceContext.Add( FString::Printf( TEXT( "%5u ***** %s" ), Line, *Lines[Line] ) );
			}
			else
			{
				CrashInfo.SourceContext.Add( FString::Printf( TEXT( "%5u       %s" ), Line, *Lines[Line] ) );
			}
		}
	}
}

void FCrashInfo::Log( FString Line )
{
	UE_LOGF( LogCrashDebugHelper, Warning, "%ls", *Line );
	Report += Line + LINE_TERMINATOR;
}

const TCHAR* FCrashInfo::GetProcessorArchitecture( EProcessorArchitecture PA )
{
	switch( PA )
	{
	case PA_X86:
		return TEXT( "x86" );
	case PA_X64:
		return TEXT( "x64" );
	case PA_ARM:
		return TEXT( "ARM" );
	}

	return TEXT( "Unknown" );
}

void FCrashInfo::WriteLine( FArchive* ReportFile, const TCHAR* Line )
{
	if (Line)
	{
		FTCHARToUTF8 UTF8Line(Line);
		ReportFile->Serialize((void*)UTF8Line.Get(), UTF8Line.Length());
	}
	ReportFile->Serialize(TCHAR_TO_UTF8(LINE_TERMINATOR), FCStringWide::Strlen(LINE_TERMINATOR));
}

void FCrashInfo::WriteLine(FArchive* ReportFile, const FString& Line)
{
	FTCHARToUTF8 UTF8Line(*Line, Line.Len());
	ReportFile->Serialize((void*)UTF8Line.Get(), UTF8Line.Length());
	ReportFile->Serialize(TCHAR_TO_UTF8(LINE_TERMINATOR), FCStringWide::Strlen(LINE_TERMINATOR));
}

void FCrashInfo::GenerateReport( const FString& DiagnosticsPath )
{
	FArchive* ReportFile = IFileManager::Get().CreateFileWriter( *DiagnosticsPath );
	if( ReportFile != NULL )
	{
		FString Line;

		WriteLine( ReportFile, TEXT( "Generating report for minidump" ) );
		WriteLine( ReportFile );

		if ( EngineVersion.Len() > 0 )
		{
			Line = FString::Printf( TEXT( "Application version %s" ), *EngineVersion );
			WriteLine( ReportFile, Line );
		}
		else if( Modules.Num() > 0 )
		{
			Line = FString::Printf( TEXT( "Application version %d.%d.%d" ), Modules[0].Major, Modules[0].Minor, Modules[0].Patch );
			WriteLine( ReportFile, Line );
		}

		Line = FString::Printf( TEXT( " ... built from changelist %d" ), BuiltFromCL );
		WriteLine( ReportFile, Line );
		if( LabelName.Len() > 0 )
		{
			Line = FString::Printf( TEXT( " ... based on label %s" ), *LabelName );
			WriteLine( ReportFile, Line );
		}
		WriteLine( ReportFile );

		Line = FString::Printf( TEXT( "OS version %d.%d.%d.%d" ), SystemInfo.OSMajor, SystemInfo.OSMinor, SystemInfo.OSBuild, SystemInfo.OSRevision );
		WriteLine( ReportFile, Line );

		Line = FString::Printf( TEXT( "Running %d %s processors" ), SystemInfo.ProcessorCount, GetProcessorArchitecture( SystemInfo.ProcessorArchitecture ) );
		WriteLine( ReportFile, Line );

		Line = FString::Printf( TEXT( "Exception was \"%s\"" ), *Exception.ExceptionString );
		WriteLine( ReportFile, Line );
		WriteLine( ReportFile );

		Line = FString::Printf( TEXT( "Source context from \"%s\"" ), *SourceFile );
		WriteLine( ReportFile, Line );
		WriteLine( ReportFile );

		Line = FString::Printf( TEXT( "<SOURCE START>" ) );
		WriteLine( ReportFile, Line );
		for( int32 LineIndex = 0; LineIndex < SourceContext.Num(); LineIndex++ )
		{
			Line = FString::Printf( TEXT( "%s" ), *SourceContext[LineIndex] );
			WriteLine( ReportFile, Line );
		}
		Line = FString::Printf( TEXT( "<SOURCE END>" ) );
		WriteLine( ReportFile, Line );
		WriteLine( ReportFile );

		Line = FString::Printf( TEXT( "<CALLSTACK START>" ) );
		WriteLine( ReportFile, Line );

		for( int32 StackIndex = 0; StackIndex < Exception.CallStackString.Num(); StackIndex++ )
		{
			Line = FString::Printf( TEXT( "%s" ), *Exception.CallStackString[StackIndex] );
			WriteLine( ReportFile, Line );
		}

		Line = FString::Printf( TEXT( "<CALLSTACK END>" ) );
		WriteLine( ReportFile, Line );
		WriteLine( ReportFile );

		Line = FString::Printf( TEXT( "%d loaded modules" ), Modules.Num() );
		WriteLine( ReportFile, Line );

		for( int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++ )
		{
			FCrashModuleInfo& Module = Modules[ModuleIndex];

			FString ModuleDirectory = FPaths::GetPath(Module.Name);
			FString ModuleName = FPaths::GetBaseFilename( Module.Name, true ) + FPaths::GetExtension( Module.Name, true );

			FString ModuleDetail = FString::Printf( TEXT( "%40s" ), *ModuleName );
			FString Version = FString::Printf( TEXT( " (%d.%d.%d.%d)" ), Module.Major, Module.Minor, Module.Patch, Module.Revision );
			ModuleDetail += FString::Printf( TEXT( " %22s" ), *Version );
			ModuleDetail += FString::Printf( TEXT( " 0x%016llx 0x%08x" ), Module.BaseOfImage, Module.SizeOfImage );
			ModuleDetail += FString::Printf( TEXT( " %s" ), *ModuleDirectory );

			WriteLine( ReportFile, ModuleDetail );
		}

		WriteLine( ReportFile );

		// Write out the processor debugging log
		WriteLine( ReportFile, Report );

		Line = FString::Printf( TEXT( "Report end!" ) );
		WriteLine( ReportFile, Line  );

		ReportFile->Close();
		delete ReportFile;
	}
}
