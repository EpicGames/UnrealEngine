// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDebugHelperModule.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "CrashDebugHelperPrivate.h"

IMPLEMENT_MODULE(FCrashDebugHelperModule, CrashDebugHelper);
DEFINE_LOG_CATEGORY(LogCrashDebugHelper);

#if PLATFORM_WINDOWS
	#include "CrashDebugHelperWindows.h"

#elif PLATFORM_LINUX
	#include "Linux/CrashDebugHelperLinux.h"

#elif PLATFORM_MAC
	#include "CrashDebugHelperMac.h"

#elif PLATFORM_IOS
	#include "CrashDebugHelperIOS.h"

#elif PLATFORM_ANDROID
	#include "Android/CrashDebugHelperAndroid.h"

#else
	#error "Unknown platform"
#endif

void FCrashDebugHelperModule::StartupModule()
{
	CrashDebugHelper = new FCrashDebugHelper();
	if (CrashDebugHelper != nullptr)
	{
		CrashDebugHelper->Init();
	}
}

void FCrashDebugHelperModule::ShutdownModule()
{
	if (CrashDebugHelper != nullptr)
	{
		delete CrashDebugHelper;
		CrashDebugHelper = nullptr;
	}
}

ICrashDebugHelper* FCrashDebugHelperModule::Get()
{
	return CrashDebugHelper;
}

ICrashDebugHelper* FCrashDebugHelperModule::GetNew()
{
	if (CrashDebugHelper != nullptr)
	{
		delete CrashDebugHelper;
		CrashDebugHelper = nullptr;
	}
	CrashDebugHelper = new FCrashDebugHelper();
	if (CrashDebugHelper != nullptr)
	{
		CrashDebugHelper->Init();
	}
	return CrashDebugHelper;
}
