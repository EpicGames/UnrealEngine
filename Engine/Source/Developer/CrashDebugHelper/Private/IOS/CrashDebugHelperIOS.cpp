// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDebugHelperIOS.h"
#include "Misc/EngineVersion.h"
#include "Apple/ApplePlatformSymbolication.h"
//#include "CrashReporter.h"
#include "CrashDebugHelperPrivate.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#if PLATFORM_IOS
#include <cxxabi.h>
#endif

#if PLATFORM_TCHAR_IS_CHAR16
#define FP_TEXT_PASTE(x) L ## x
#define WTEXT(x) FP_TEXT_PASTE(x)
#else
#define WTEXT TEXT
#endif

FString ExtractRelativePath( const TCHAR* BaseName, TCHAR const* FullName )
{
	FString FullPath = FString( FullName ).ToLower();
	FullPath.ReplaceInline( TEXT( "\\" ), TEXT( "/" ) );
	
	TArray<FString> Components;
	int32 Count = FullPath.ParseIntoArray( Components, TEXT( "/" ), true );
	FullPath = TEXT( "" );
	
	for( int32 Index = 0; Index < Count; Index++ )
	{
		if( Components[Index] == BaseName )
		{
			if( Index > 0 )
			{
				for( int32 Inner = Index - 1; Inner < Count; Inner++ )
				{
					FullPath += Components[Inner];
					if( Inner < Count - 1 )
					{
						FullPath += TEXT( "/" );
					}
				}
			}
			break;
		}
	}
	
	return FullPath;
}

static int32 ParseReportVersion(TCHAR const* CrashLog, int32& OutVersion)
{
	int32 Found = 0;
	TCHAR const* VersionLine = FCStringWide::Strstr(CrashLog, TEXT("Report Version:"));
	if (VersionLine)
	{
		Found = swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*ls %*ls %d"), &OutVersion);
	}
	return Found;
}

static int32 ParseVersion(TCHAR const* CrashLog, int32& OutMajor, int32& OutMinor, int32& OutBuild, int32& OutChangeList, FString& OutBranch)
{
	int32 Found = 0;
	TCHAR const* VersionLine = FCStringWide::Strstr(CrashLog, TEXT("Version:"));
	if (VersionLine)
	{
		TCHAR Branch[257] = {0};
		Found = swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*s %d.%d.%d (%*d.%*d.%*d-%d+%256ls)"), &OutMajor, &OutMinor, &OutBuild, &OutChangeList, Branch);
		if(Found == 5)
		{
			TCHAR* BranchEnd = FCStringWide::Strchr(Branch, TEXT(')'));
			if(BranchEnd)
			{
				*BranchEnd = TEXT('\0');
			}
			OutBranch = Branch;
		}
	}
	return Found;
}

static int32 ParseOS(TCHAR const* CrashLog, uint16& OutMajor, uint16& OutMinor, uint16& OutPatch, uint16& OutBuild)
{
	int32 Found = 0;
	TCHAR const* VersionLine = FCStringWide::Strstr(CrashLog, TEXT("OS Version:"));
	if (VersionLine)
	{
		Found = swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*s %*s iPhone OS %hd.%hd.%hd (%hxd)"), &OutMajor, &OutMinor, &OutPatch, &OutBuild);
        if ( Found == 2 )
        {
            OutPatch = 0;
            Found += swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*s %*s iPhone OS %*hd.%*hd (%hxd)"), &OutBuild) + 1;
        }
	}
	return Found;
}

static bool ParseModel(TCHAR const* CrashLog, FString& OutModelDetails, uint32& OutProcessorNum)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Model:"));
	if (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Model: "));
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(256, (int32)((uintptr_t)(End - Line)));
		OutModelDetails.Append(Line, Length);
		
		OutProcessorNum = 1;
		int32 ProcessorPos = OutModelDetails.Find(TEXT(" processors"));
		if( ProcessorPos != INDEX_NONE )
		{
			int32 NumStart = ProcessorPos;
			while(NumStart && OutModelDetails[NumStart] != TEXT(','))
			{
				NumStart--;
			}
			if(NumStart >= 0 && OutModelDetails[NumStart] == TEXT(','))
			{
				NumStart += 2;
				FString NumProc = OutModelDetails.Mid(NumStart, ProcessorPos-NumStart);
				if(NumProc.IsNumeric())
				{
					TTypeFromString<uint32>::FromString(OutProcessorNum, *NumProc);
				}
			}
		}
		
		bFound = true;
	}
	return bFound;
}

static int32 ParseGraphics(TCHAR const* CrashLog, FString& OutGPUDetails)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Graphics:"));
	int32 Output = 0;
	while (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Graphics:"));
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		OutGPUDetails.Append(TEXT(", "));
		int32 Length = FMath::Min((256 - Output), (int32)((uintptr_t)(End - Line)));
		OutGPUDetails.Append(Line, Length);
		
		Line = FCStringWide::Strstr(Line, TEXT("Graphics:"));
		
		bFound = true;
	}
	return bFound;
}

static int32 ParseError(TCHAR const* CrashLog, FString& OutErrorDetails)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Exception Codes:"));
	if (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Exception Codes:"));
		check(Line);
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Line)));
		OutErrorDetails.Append(Line, Length);
		
		bFound = true;
	}
	Line = FCStringWide::Strstr(CrashLog, TEXT("Application Specific Information:"));
	if (Line)
	{
		Line = FCStringWide::Strchr(Line, TEXT('\n'));
		check(Line);
		Line += 1;
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Line)));
		OutErrorDetails += TEXT(" ");
		OutErrorDetails.Append(Line, Length);
		
		bFound = true;
	}
	return bFound;
}

static int32 ParseExceptionCode(TCHAR const* CrashLog, uint32& OutExceptionCode)
{
	int32 Found = 0;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Exception Type:"));
	if(Line)
	{
		wchar_t BufferW[257] = {0};
		Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s %*s (%256ls)"), BufferW);
		if(!Found)
		{
			Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s %256ls"), BufferW);
		}
		if(Found)
		{
			TCHAR* End = FCStringWide::Strchr(WCHAR_TO_TCHAR(BufferW), TEXT(')'));
			if(End)
			{
				*End = TEXT('\0');
			}
			if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGQUIT")) == 0)
			{
				OutExceptionCode = SIGQUIT;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGILL")) == 0)
			{
				OutExceptionCode = SIGILL;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGEMT")) == 0)
			{
				OutExceptionCode = SIGEMT;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGFPE")) == 0)
			{
				OutExceptionCode = SIGFPE;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGBUS")) == 0)
			{
				OutExceptionCode = SIGBUS;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGSEGV")) == 0)
			{
				OutExceptionCode = SIGSEGV;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGSYS")) == 0)
			{
				OutExceptionCode = SIGSYS;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGABRT")) == 0)
			{
				OutExceptionCode = SIGABRT;
			}
			else if(FCStringWide::Strcmp(WCHAR_TO_TCHAR(BufferW), TEXT("SIGTRAP")) == 0)
			{
				OutExceptionCode = SIGTRAP;
			}
			else if(FString(WCHAR_TO_TCHAR(BufferW)).IsNumeric())
			{
				Found = swscanf(BufferW, WTEXT("%u"), &OutExceptionCode);
			}
			else
			{
				ensure(false);
				OutExceptionCode = SIGUSR1;
			}
		}
	}
	return Found;
}

static int32 ParseCrashedThread(TCHAR const* CrashLog, uint32& OutThreadNumber)
{
	int32 Found = 0;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Crashed Thread:"));
	if (Line)
	{
		Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s %u"), &OutThreadNumber);
	}
	return Found;
}

static int32 ParseProcessID(TCHAR const* CrashLog, uint32& OutPID)
{
	int32 Found = 0;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Process:"));
	if (Line)
	{
		Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s [%u]"), &OutPID);
	}
	return Found;
}

static TCHAR const* FindThreadStack(TCHAR const* CrashLog, uint32 const ThreadNumber)
{
	int32 Found = 0;
	FString Format = FString::Printf(TEXT("Thread %u"), ThreadNumber);
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, *Format);
	if (Line)
	{
		Line = FCStringWide::Strchr(Line, TEXT('\n'));
		check(Line);
		Line += 1;
	}
	return Line;
}

static TCHAR const* FindCrashedThreadStack(TCHAR const* CrashLog)
{
	TCHAR const* Line = nullptr;
	uint32 ThreadNumber = 0;
	int32 Found = ParseCrashedThread(CrashLog, ThreadNumber);
	if(Found)
	{
		Line = FindThreadStack(CrashLog, ThreadNumber);
	}
	return Line;
}

static int32 ParseThreadStackLine(TCHAR const* StackLine, FString& OutModuleName, uint64& OutProgramCounter, FString& OutFunctionName, FString& OutFileName, int32& OutLineNumber, uint64& OutFuncAddress, uint64& OutFuncOffset)
{
	wchar_t ModuleNameW[257];
	wchar_t FunctionNameW[1025];
	wchar_t FileNameW[257];
	
	int32 Found = swscanf(TCHAR_TO_WCHAR(StackLine), WTEXT("%*d %256ls 0x%lx"), ModuleNameW, &OutProgramCounter);
	if(Found == 2)
	{
		OutFuncAddress = 0;
		OutFuncOffset = 0;
		if(swscanf(TCHAR_TO_WCHAR(StackLine), WTEXT("%*d %*ls 0x%*lx 0x%lx + %d"), &OutFuncAddress, &OutFuncOffset) == 0)
		{
			Found += swscanf(TCHAR_TO_WCHAR(StackLine), WTEXT("%*d %*ls 0x%*lx %1024ls + %*d (%256ls:%d)"), FunctionNameW, FileNameW, &OutLineNumber);
		}
	}
		
    switch(Found)
    {
        case 5:
        case 4:
        {
            OutFileName = WCHAR_TO_TCHAR(FileNameW);
        }
        case 3:
        {
#if PLATFORM_IOS
            int32 Status = -1;
            ANSICHAR* DemangledName = abi::__cxa_demangle(TCHAR_TO_UTF8(WCHAR_TO_TCHAR(FunctionNameW)), nullptr, nullptr, &Status);
            if (DemangledName && Status == 0)
            {
                // C++ function
                OutFunctionName = FString::Printf(TEXT("%ls "), UTF8_TO_TCHAR(DemangledName));
            }
            else
#endif
            if (FCStringWide::Strlen(WCHAR_TO_TCHAR(FunctionNameW)) > 0 && FCStringWide::Strchr(WCHAR_TO_TCHAR(FunctionNameW), ']'))
            {
                // ObjC function
                OutFunctionName = FString::Printf(TEXT("%ls "), WCHAR_TO_TCHAR(FunctionNameW));
            }
            else if (FCStringWide::Strlen(WCHAR_TO_TCHAR(FunctionNameW)) > 0)
            {
                // C Function
                OutFunctionName = FString::Printf(TEXT("%ls() "), WCHAR_TO_TCHAR(FunctionNameW));
            }
        }
        case 2:
        case 1:
        {
            OutModuleName = WCHAR_TO_TCHAR(ModuleNameW);
        }
        default:
        {
            break;
        }
    }
	return Found;
}

static int32 SymboliseStackInfo(FPlatformSymbolDatabaseSet& SymbolCache, TArray<FCrashModuleInfo> const& ModuleInfo, FString ModuleName, uint64 const ProgramCounter, FString& OutFunctionName, FString& OutFileName, int32& OutLineNumber)
{
	FProgramCounterSymbolInfo Info;
	int32 ValuesSymbolised = 0;
	
	FCrashModuleInfo Module;
	for (auto Iterator : ModuleInfo)
	{
		if(Iterator.Name.EndsWith(ModuleName))
		{
			Module = Iterator;
			break;
		}
	}
	
	FApplePlatformSymbolDatabase* Db = SymbolCache.Find(Module.Report);
	if(!Db)
	{
		FApplePlatformSymbolDatabase Database;
		if(FPlatformSymbolication::LoadSymbolDatabaseForBinary(TEXT(""), Module.Name, Module.Report, {}, Database))
		{
			SymbolCache.Add(Database);
			Db = SymbolCache.Find(Module.Report);
		}
        else
        {
            // add a dummy DB so we don't keep trying to load one that isn't correct.
            FApplePlatformSymbolDatabase DB;
            DB.GenericDB->Signature = Module.Report;
            SymbolCache.Add(DB);
            Db = SymbolCache.Find(Module.Report);
        }
	}
	if((Module.Name.Len() > 0) && Db && FPlatformSymbolication::SymbolInfoForStrippedSymbol(*Db, ProgramCounter, Module.BaseOfImage, Module.Report, Info))
	{
		if(FCStringAnsi::Strlen(Info.FunctionName) > 0)
		{
			OutFunctionName = Info.FunctionName;
			ValuesSymbolised++;
		}
		if(ValuesSymbolised == 1 && FCStringAnsi::Strlen(Info.Filename) > 0)
		{
			OutFileName = Info.Filename;
			ValuesSymbolised++;
		}
		if(ValuesSymbolised == 2 && Info.LineNumber > 0)
		{
			OutLineNumber = Info.LineNumber;
			ValuesSymbolised++;
		}
	}
	
	return ValuesSymbolised;
}

static TCHAR const* FindModules(TCHAR const* CrashLog)
{
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Binary Images:"));
	if (Line)
	{
		Line = FCStringWide::Strchr(Line, TEXT('\n'));
		check(Line);
		Line += 1;
	}
	return Line;
}

static int32 ParseModuleVersion(TCHAR const* Version, uint16& OutMajor, uint16& OutMinor, uint16& OutPatch, uint16& OutBuild)
{
	OutMajor = OutMinor = OutPatch = OutBuild = 0;
	int32 Found = swscanf(TCHAR_TO_WCHAR(Version), WTEXT("%hu.%hu.%hu"), &OutMajor, &OutMinor, &OutPatch);
	TCHAR const* CurrentStart = FCStringWide::Strchr(Version, TEXT('-'));
	if(CurrentStart)
	{
		int32 Components[3] = {0, 0, 0};
		int32 Result = swscanf(TCHAR_TO_WCHAR(CurrentStart), WTEXT("%*ls %d.%d.%d"), &Components[0], &Components[1], &Components[2]);
		OutBuild = (uint16)(Components[0] * 10000) + (Components[1] * 100) + (Components[2]);
		Found = 4;
	}
	return Found;
}

static bool ParseModuleLine(TCHAR const* ModuleLine, FCrashModuleInfo& OutModule)
{
	bool bOK = false;
	TCHAR ModuleName[257] = {0};
	uint64 ModuleBase = 0;
	uint64 ModuleEnd = 0;
	
	int32 Found = swscanf(TCHAR_TO_WCHAR(ModuleLine), WTEXT("%lx %*ls %lx %256ls"), &ModuleBase, &ModuleEnd, ModuleName);
	switch (Found)
	{
		case 3:
		{
			TCHAR const* VersionStart = FCStringWide::Strchr(ModuleLine, TEXT('('));
			TCHAR const* VersionEnd = FCStringWide::Strchr(ModuleLine, TEXT(')'));
			if(VersionStart && VersionEnd)
			{
				++VersionStart;
				Found += ParseModuleVersion(VersionStart, OutModule.Major, OutModule.Minor, OutModule.Patch, OutModule.Revision);
			}
			
			TCHAR const* UUIDStart = FCStringWide::Strchr(ModuleLine, TEXT('<'));
			TCHAR const* UUIDEnd = FCStringWide::Strchr(ModuleLine, TEXT('>'));
			if(UUIDStart && UUIDEnd)
			{
				++UUIDStart;
				int32 Length = FMath::Min(64, (int32)((uintptr_t)(UUIDEnd - UUIDStart)));
				OutModule.Report.Append(UUIDStart, Length);
				if(!OutModule.Report.Contains(TEXT("-")))
				{
					OutModule.Report.InsertAt(8, TEXT('-'));
					OutModule.Report.InsertAt(13, TEXT('-'));
					OutModule.Report.InsertAt(18, TEXT('-'));
					OutModule.Report.InsertAt(23, TEXT('-'));
				}
				OutModule.Report = OutModule.Report.ToUpper();
				Found++;
			}
			
			TCHAR const* Path = FCStringWide::Strchr(ModuleLine, TEXT('/'));
			if(Path)
			{
				TCHAR const* End = FCStringWide::Strchr(Path, TEXT('\r'));
				if(!End)
				{
					End = FCStringWide::Strchr(Path, TEXT('\n'));
				}
				check(End);
				
				int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Path)));
				OutModule.Name.Append(Path, Length);
				Found++;
				bOK = true;
			}
		}
		case 2:
		{
			OutModule.SizeOfImage = (ModuleBase - ModuleEnd);
		}
		case 1:
		{
			OutModule.BaseOfImage = ModuleBase;
			break;
		}
		default:
		{
			break;
		}
	}
	return bOK;
}

FCrashDebugHelperIOS::FCrashDebugHelperIOS()
{
}

FCrashDebugHelperIOS::~FCrashDebugHelperIOS()
{
}

bool FCrashDebugHelperIOS::ParseCrashDump(const FString& InCrashDumpName, FCrashDebugInfo& OutCrashDebugInfo)
{
	if (bInitialized == false)
	{
		UE_LOGF(LogCrashDebugHelper, Warning, "ParseCrashDump: CrashDebugHelper not initialized");
		return false;
	}
	
	FString CrashDump;
	if ( FFileHelper::LoadFileToString( CrashDump, *InCrashDumpName ) )
	{
		// Only supports Apple crash report version 11
		int32 ReportVersion = 0;
		int32 Result = ParseReportVersion(*CrashDump, ReportVersion);
		if(Result == 1 && (ReportVersion == 11 || ReportVersion == 104))
		{
			int32 Major = 0;
			int32 Minor = 0;
			int32 Build = 0;
			int32 CLNumber = 0;
			FString Branch;
			Result = ParseVersion(*CrashDump, Major, Minor, Build, CLNumber, Branch);
			if(Result >= 1)
			{
                if (Result < 3)
                {
                    OutCrashDebugInfo.EngineVersion = Major;
                }
				else if (Result < 5)
				{
					OutCrashDebugInfo.EngineVersion = Build;
				}
				else
				{
					OutCrashDebugInfo.EngineVersion = CLNumber;
				}
				if(Result == 5)
				{
					OutCrashDebugInfo.SourceControlLabel = Branch;
				}
				OutCrashDebugInfo.PlatformName = TEXT("IOS");
				OutCrashDebugInfo.CrashDumpName = InCrashDumpName;
				return true;
			}
		}
	}

	return false;
}

bool FCrashDebugHelperIOS::CreateMinidumpDiagnosticReport( const FString& InCrashDumpName )
{
	bool bOK = false;
	
	FString CrashDump;
	if ( FFileHelper::LoadFileToString( CrashDump, *InCrashDumpName ) )
	{
		int32 ReportVersion = 0;
		int32 Result = ParseReportVersion(*CrashDump, ReportVersion);
		if(Result == 1 && (ReportVersion == 11 || ReportVersion == 104))
		{
			FString Error;
			FString ModulePath;
			FString ModuleName;
			FString FunctionName;
			FString FileName;
			FString Branch;
			FString Model;
			FString Gpu;
			
			uint64 ProgramCounter = 0;
			uint64 FunctionAddress = 0;
			uint64 FunctionOffset = 0;
			int32 Major = 0;
			int32 Minor = 0;
			int32 Build = 0;
			int32 CLNumber = 0;
			int32 LineNumber = 0;;
			
			Result = ParseVersion(*CrashDump, Major, Minor, Build, CLNumber, Branch);
			if(Result >= 3)
			{
				CrashInfo.EngineVersion = FEngineVersion(Major, Minor, Build, CLNumber, Branch).ToString();
			}
			
			if(Result >= 4)
			{
				CrashInfo.BuiltFromCL = CLNumber;
			}
			
			if(Result == 5 && Branch.Len() > 0)
			{
				CrashInfo.LabelName = Branch;
			}
			
			Result = ParseOS(*CrashDump, CrashInfo.SystemInfo.OSMajor, CrashInfo.SystemInfo.OSMinor, CrashInfo.SystemInfo.OSBuild, CrashInfo.SystemInfo.OSRevision);
			check(Result == 4);
			
			CrashInfo.SystemInfo.ProcessorArchitecture = PA_X64;
			
			ParseModel(*CrashDump, Model, CrashInfo.SystemInfo.ProcessorCount);
			ParseGraphics(*CrashDump, Gpu);
			CrashInfo.SystemInfo.Report = Model + Gpu;
			
			Result = ParseError(*CrashDump, CrashInfo.Exception.ExceptionString);
			check(Result == 1);
			
			Result = ParseProcessID(*CrashDump, CrashInfo.Exception.ProcessId);
			check(Result == 1);
			
			Result = ParseCrashedThread(*CrashDump, CrashInfo.Exception.ThreadId);
			check(Result == 1);
			
			Result = ParseExceptionCode(*CrashDump, CrashInfo.Exception.Code);
			check(Result == 1);
			
			// Parse modules now for symbolication - if we don't have the running process we need to symbolicate by UUID
			TCHAR const* ModuleLine = FindModules(*CrashDump);
			while(ModuleLine)
			{
				FCrashModuleInfo Module;
				if (ParseModuleLine(ModuleLine, Module))
				{
					CrashInfo.Modules.Push(Module);
					CrashInfo.ModuleNames.Push(FPaths::GetBaseFilename(Module.Name));
					
					ModuleLine = FCStringWide::Strchr(ModuleLine, TEXT('\n'));
					check(ModuleLine);
					ModuleLine += 1;
				}
				else
				{
					ModuleLine = nullptr;
				}
			}
			
			
			FPlatformSymbolDatabaseSet SymbolCache;
			
			
			uint32 CrashedThreadNumber = 0;
			ParseCrashedThread(*CrashDump, CrashedThreadNumber);
			
			for (int ThreadNumber = 0; true; ++ThreadNumber)
			{
				TCHAR const* ThreadStackLine = FindThreadStack(*CrashDump, ThreadNumber);
				if (!ThreadStackLine)
				{
					break;
				}
				
				FCrashThreadInfo ThreadInfo;
				ThreadInfo.SuspendCount = 0;
				ThreadInfo.IsCrashing = (CrashedThreadNumber == ThreadNumber);
				ThreadInfo.ThreadId = ThreadNumber; // PLReporter does not preserve ThreadId, just use index number
				ThreadInfo.ThreadName = FString::Printf(TEXT("Thread %d"), ThreadInfo.ThreadId); // PLReporter does not preserve ThreadName either
				
				bool bIsCrashLocation = true;
				uint32 Index = 0;
				while(ThreadStackLine)
				{
					if(ThreadInfo.IsCrashing && CrashInfo.Exception.Code == SIGTRAP)
					{
						// For ensures strip the first three lines as they are PLCrashReporter nonsense
						if(Index < 3)
						{
							ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
							if(ThreadStackLine)
							{
								ThreadStackLine += 1;
							}
							++Index;
							continue;
						}
						
						// Crash location is the 5th entry in the stack.
						bIsCrashLocation = (Index == 5);
					}
					
					Result = ParseThreadStackLine(ThreadStackLine, ModuleName, ProgramCounter, FunctionName, FileName, LineNumber, FunctionAddress, FunctionOffset);
					
					// If we got the modulename & program counter but didn't parse the filename & linenumber we can resymbolise
					if(Result > 1 && Result < 4)
					{
						// Attempt to resymbolise using CoreSymbolication
						Result += SymboliseStackInfo(SymbolCache, CrashInfo.Modules, ModuleName, ProgramCounter, FunctionName, FileName, LineNumber);
					}
					
					if (ThreadInfo.IsCrashing)
					{
						// Output in our format based on the fields we actually have
						CrashInfo.Exception.PortableCallStackString.Push( FString::Printf(TEXT("%-*s 0x%016llx + %-16llx"), 40, *ModuleName, FunctionAddress, FunctionOffset));
					}
					ThreadInfo.PortableCallStackString += FString::Printf(TEXT("%-*s 0x%016llx + %-16llx\n"), 40, *ModuleName, FunctionAddress, FunctionOffset);
					
					switch (Result)
					{
						case 2:
							if (ThreadInfo.IsCrashing)
							{
								CrashInfo.Exception.CallStackString.Push( FString::Printf( TEXT( "Unknown() Address = 0x%lx (filename not found) [in %s]" ), ProgramCounter, *ModuleName ) );
							}
							ThreadInfo.CallStack.Push(ProgramCounter);
							
							ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
							check(ThreadStackLine);
							ThreadStackLine += 1;
							break;
							
						case 3:
						case 4:
							if (ThreadInfo.IsCrashing)
							{
								CrashInfo.Exception.CallStackString.Push( FString::Printf( TEXT( "%s Address = 0x%lx (filename not found) [in %s]" ), *FunctionName, ProgramCounter, *ModuleName ) );
							}
							ThreadInfo.CallStack.Push(ProgramCounter);
							
							ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
							check(ThreadStackLine);
							ThreadStackLine += 1;
							break;
							
						case 5:
						case 6: // Function name might be parsed twice
							if(ThreadInfo.IsCrashing && bIsCrashLocation)
							{
								if( FileName.Len() > 0 && LineNumber > 0 )
								{
									// Sync the source file where the crash occurred
									CrashInfo.SourceFile = ExtractRelativePath( TEXT( "source" ), *FileName );
									CrashInfo.SourceLineNumber = LineNumber;
									
									// Add the standard source context
									AddSourceToReport();
								}
							}
							
							if (ThreadInfo.IsCrashing)
							{
								CrashInfo.Exception.CallStackString.Push( FString::Printf( TEXT( "%s Address = 0x%lx [%s, line %d] [in %s]" ), *FunctionName, ProgramCounter, *FileName, LineNumber, *ModuleName ) );
							}
							ThreadInfo.CallStack.Push(ProgramCounter);
							
							ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
							check(ThreadStackLine);
							ThreadStackLine += 1;
							break;
							
						default:
							ThreadStackLine = nullptr;
							break;
					}
					
					++Index;
					bIsCrashLocation = false;
				}
				
				CrashInfo.Threads.Push(ThreadInfo);
			}
			
			TCHAR const* ThreadStackLine = FindCrashedThreadStack(*CrashDump);
			
			
			
			bOK = true;
		}
	}
	
	return bOK;
}
