// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDebugHelperMac.h"
#include "Misc/EngineVersion.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/ApplePlatformSymbolication.h"
#include "CrashReporter.h"
#include "CrashDebugHelperPrivate.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include <cxxabi.h>

#if PLATFORM_TCHAR_IS_CHAR16
#define FP_TEXT_PASTE(x) L ## x
#define WTEXT(x) FP_TEXT_PASTE(x)
#else
#define WTEXT TEXT
#endif

FString ExtractRelativePath( const TCHAR* BaseName, TCHAR const* FullName )
{
	FString FullPath = FString( FullName ).ToLower();
	FullPath.ReplaceInline( TEXT( "\\" ), TEXT( "/" ) );
	
	TArray<FString> Components;
	int32 Count = FullPath.ParseIntoArray( Components, TEXT( "/" ), true );
	FullPath = TEXT( "" );
	
	for( int32 Index = 0; Index < Count; Index++ )
	{
		if( Components[Index] == BaseName )
		{
			if( Index > 0 )
			{
				for( int32 Inner = Index - 1; Inner < Count; Inner++ )
				{
					FullPath += Components[Inner];
					if( Inner < Count - 1 )
					{
						FullPath += TEXT( "/" );
					}
				}
			}
			break;
		}
	}
	
	return FullPath;
}

static int32 ParseReportVersion(TCHAR const* CrashLog, int32& OutVersion)
{
	int32 Found = 0;
	TCHAR const* VersionLine = FCStringWide::Strstr(CrashLog, TEXT("Report Version:"));
	if (VersionLine)
	{
		Found = swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*ls %*ls %d"), &OutVersion);
	}
	return Found;
}

static int32 ParseVersion(TCHAR const* CrashLog, int32& OutMajor, int32& OutMinor, int32& OutBuild, int32& OutChangeList, FString& OutBranch)
{
	int32 Found = 0;
	TCHAR const* VersionLine = FCStringWide::Strstr(CrashLog, TEXT("Version:"));
	if (VersionLine)
	{
		wchar_t RawBranch[257] = {0};
		Found = swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*s %d.%d.%d (%*d.%*d.%*d-%d+%256ls)"), &OutMajor, &OutMinor, &OutBuild, &OutChangeList, RawBranch);
		if(Found == 5)
		{
			TCHAR* BranchEnd = FCStringWide::Strchr(WCHAR_TO_TCHAR(RawBranch), TEXT(')'));
			if(BranchEnd)
			{
				*BranchEnd = TEXT('\0');
			}
			OutBranch = FString(WCHAR_TO_TCHAR(RawBranch));
		}
	}
	return Found;
}

static int32 ParseOS(TCHAR const* CrashLog, uint16& OutMajor, uint16& OutMinor, uint16& OutPatch, uint16& OutBuild)
{
	int32 Found = 0;
	TCHAR const* VersionLine = FCStringWide::Strstr(CrashLog, TEXT("OS Version:"));
	if (VersionLine)
	{
		Found = swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*s %*s Mac OS X %hd.%hd.%hd (%hxd)"), &OutMajor, &OutMinor, &OutPatch, &OutBuild);
        if ( Found == 2 )
        {
            OutPatch = 0;
            Found += swscanf(TCHAR_TO_WCHAR(VersionLine), WTEXT("%*s %*s Mac OS X %*hd.%*hd (%hxd)"), &OutBuild) + 1;
        }
	}
	return Found;
}

static bool ParseModel(TCHAR const* CrashLog, FString& OutModelDetails, uint32& OutProcessorNum)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Model:"));
	if (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Model: "));
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(256, (int32)((uintptr_t)(End - Line)));
		OutModelDetails.Append(Line, Length);
		
		OutProcessorNum = 1;
		int32 ProcessorPos = OutModelDetails.Find(TEXT(" processors"));
		if( ProcessorPos != INDEX_NONE )
		{
			int32 NumStart = ProcessorPos;
			while(NumStart && OutModelDetails[NumStart] != TEXT(','))
			{
				NumStart--;
			}
			if(NumStart >= 0 && OutModelDetails[NumStart] == TEXT(','))
			{
				NumStart += 2;
				FString NumProc = OutModelDetails.Mid(NumStart, ProcessorPos-NumStart);
				if(NumProc.IsNumeric())
				{
					TTypeFromString<uint32>::FromString(OutProcessorNum, *NumProc);
				}
			}
		}
		
		bFound = true;
	}
	return bFound;
}

static int32 ParseGraphics(TCHAR const* CrashLog, FString& OutGPUDetails)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Graphics:"));
	int32 Output = 0;
	while (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Graphics:"));
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		OutGPUDetails.Append(TEXT(", "));
		int32 Length = FMath::Min((256 - Output), (int32)((uintptr_t)(End - Line)));
		OutGPUDetails.Append(Line, Length);
		
		Line = FCStringWide::Strstr(Line, TEXT("Graphics:"));
		
		bFound = true;
	}
	return bFound;
}

static int32 ParseArchitecture(TCHAR const* CrashLog, FString& Architecture)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Code Type:"));
	int32 Output = 0;
	while (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Code Type:"));
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Line)));
		Architecture.Append(Line, Length);
		Architecture = Architecture.TrimStartAndEnd();
		Line = nullptr;
		bFound = true;
	}
	return bFound;
}

static int32 ParseError(TCHAR const* CrashLog, FString& OutErrorDetails)
{
	bool bFound = false;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Exception Codes:"));
	if (Line)
	{
		Line += FCStringWide::Strlen(TEXT("Exception Codes:"));
		check(Line);
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Line)));
		OutErrorDetails.Append(Line, Length);
		
		bFound = true;
	}
	Line = FCStringWide::Strstr(CrashLog, TEXT("Application Specific Information:"));
	if (Line)
	{
		Line = FCStringWide::Strchr(Line, TEXT('\n'));
		check(Line);
		Line += 1;
		TCHAR const* End = FCStringWide::Strchr(Line, TEXT('\r'));
		if(!End)
		{
			End = FCStringWide::Strchr(Line, TEXT('\n'));
		}
		check(End);
		
		int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Line)));
		OutErrorDetails += TEXT(" ");
		OutErrorDetails.Append(Line, Length);
		
		bFound = true;
	}
	return bFound;
}

static int32 ParseExceptionCode(TCHAR const* CrashLog, uint32& OutExceptionCode)
{
	int32 Found = 0;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Exception Type:"));
	if(Line)
	{
		wchar_t RawBuffer[257] = {0};
		Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s %*s (%256ls)"), RawBuffer);
		if(!Found)
		{
			Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s %256ls"), RawBuffer);
		}
		if(Found)
		{
			FString Buffer = FString(WCHAR_TO_TCHAR(RawBuffer));

			int End = Buffer.Find(TEXT(")"));
			if(End != INDEX_NONE)
			{
				Buffer = Buffer.LeftChop(End);
			}
			if(Buffer.Find(TEXT("SIGQUIT")) != INDEX_NONE)
			{
				OutExceptionCode = SIGQUIT;
			}
			else if(Buffer.Find(TEXT("SIGILL")) != INDEX_NONE)
			{
				OutExceptionCode = SIGILL;
			}
			else if(Buffer.Find(TEXT("SIGEMT")) != INDEX_NONE)
			{
				OutExceptionCode = SIGEMT;
			}
			else if(Buffer.Find(TEXT("SIGFPE")) != INDEX_NONE)
			{
				OutExceptionCode = SIGFPE;
			}
			else if(Buffer.Find(TEXT("SIGBUS")) != INDEX_NONE)
			{
				OutExceptionCode = SIGBUS;
			}
			else if(Buffer.Find(TEXT("SIGSEGV")) != INDEX_NONE)
			{
				OutExceptionCode = SIGSEGV;
			}
			else if(Buffer.Find(TEXT("SIGSYS")) != INDEX_NONE)
			{
				OutExceptionCode = SIGSYS;
			}
			else if(Buffer.Find(TEXT("SIGABRT")) != INDEX_NONE)
			{
				OutExceptionCode = SIGABRT;
			}
			else if(Buffer.Find(TEXT("SIGTRAP")) != INDEX_NONE)
			{
				OutExceptionCode = SIGTRAP;
			}
			else if(Buffer.Find(TEXT("SIGUSR1")) != INDEX_NONE)
			{
				OutExceptionCode = SIGUSR1;
			}
			else if(Buffer.Find(TEXT("SIGUSR2")) != INDEX_NONE)
			{
				OutExceptionCode = SIGUSR2;
			}
			else if(Buffer.IsNumeric())
			{
				Found = swscanf(TCHAR_TO_WCHAR(*Buffer), WTEXT("%u"), &OutExceptionCode);
			}
			else
			{
				UE_LOGF(LogCrashDebugHelper, Warning, "Unrecognized exception code: %ls", *Buffer);
				OutExceptionCode = SIGUSR1;
			}
		}
	}
	return Found;
}

static int32 ParseCrashedThread(TCHAR const* CrashLog, uint32& OutThreadNumber)
{
	int32 Found = 0;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Crashed Thread:"));
	if (Line)
	{
		Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s %u"), &OutThreadNumber);
	}
	return Found;
}

static int32 ParseProcessID(TCHAR const* CrashLog, uint32& OutPID)
{
	int32 Found = 0;
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Process:"));
	if (Line)
	{
		Found = swscanf(TCHAR_TO_WCHAR(Line), WTEXT("%*s %*s [%u]"), &OutPID);
	}
	return Found;
}

static TCHAR const* FindThreadStack(TCHAR const* CrashLog, uint32 const ThreadNumber)
{
	int32 Found = 0;
	FString Format = FString::Printf(TEXT("Thread %u"), ThreadNumber);
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, *Format);
	if (Line)
	{
		Line = FCStringWide::Strchr(Line, TEXT('\n'));
		check(Line);
		Line += 1;
	}
	return Line;
}

static TCHAR const* FindCrashedThreadStack(TCHAR const* CrashLog)
{
	TCHAR const* Line = nullptr;
	uint32 ThreadNumber = 0;
	int32 Found = ParseCrashedThread(CrashLog, ThreadNumber);
	if(Found)
	{
		Line = FindThreadStack(CrashLog, ThreadNumber);
	}
	return Line;
}

static int32 ParseThreadStackLine(TCHAR const* StackLine, FString& OutModuleName, uint64& OutProgramCounter, FString& OutFunctionName, FString& OutFileName, int32& OutLineNumber)
{
	wchar_t RawModuleName[257];
	wchar_t RawFunctionName[1025];
	wchar_t RawFileName[257];
	
	int32 Found = swscanf(TCHAR_TO_WCHAR(StackLine), WTEXT("%*d %256ls 0x%lx"), RawModuleName, &OutProgramCounter);
	if(Found == 2)
	{
		uint64 FunctionAddress = 0;
		uint32 FunctionOffset = 0;
		if(swscanf(TCHAR_TO_WCHAR(StackLine), WTEXT("%*d %*ls %*lx 0x%lx + %d"), &FunctionAddress, &FunctionOffset) == 0)
		{
			Found += swscanf(TCHAR_TO_WCHAR(StackLine), WTEXT("%*d %*ls %*lx %1024ls + %*d (%256ls:%d)"), RawFunctionName, RawFileName, &OutLineNumber);
		}
	}
	
	switch(Found)
	{
		case 5:
		case 4:
		{
			OutFileName = WCHAR_TO_TCHAR(RawFileName);
		}
		case 3:
		{
			int32 Status = -1;
			FString FunctionName = FString(WCHAR_TO_TCHAR(RawFunctionName));
			ANSICHAR* DemangledName = abi::__cxa_demangle(TCHAR_TO_UTF8(*FunctionName), nullptr, nullptr, &Status);
			if(DemangledName && Status == 0)
			{
				// C++ function
				OutFunctionName = FString::Printf(TEXT("%ls "), UTF8_TO_TCHAR(DemangledName));
			}
			else if (FCStringWide::Strlen(*FunctionName) > 0 && FCStringWide::Strchr(*FunctionName, ']'))
			{
				// ObjC function
				OutFunctionName = FString::Printf(TEXT("%ls "), *FunctionName);
			}
			else if(FCStringWide::Strlen(*FunctionName) > 0)
			{
				// C function
				OutFunctionName = FString::Printf(TEXT("%ls() "), *FunctionName);
			}
		}
		case 2:
		case 1:
		{
			OutModuleName = WCHAR_TO_TCHAR(RawModuleName);
		}
		default:
		{
			break;
		}
	}
	
	return Found;
}

static int32 SymboliseStackInfo(FPlatformSymbolDatabaseSet& SymbolCache, TArray<FCrashModuleInfo> const& ModuleInfo, FString ModuleName, FString Architecture, uint64 const ProgramCounter, FString& OutFunctionName, FString& OutFileName, int32& OutLineNumber)
{
	FProgramCounterSymbolInfo Info;
	int32 ValuesSymbolised = 0;
	
	FCrashModuleInfo Module;
	for (auto Iterator : ModuleInfo)
	{
		if(Iterator.Name.EndsWith(ModuleName))
		{
			Module = Iterator;
			break;
		}
	}
	
	FApplePlatformSymbolDatabase* Db = SymbolCache.Find(Module.Report);
	if(!Db)
	{
		FApplePlatformSymbolDatabase Database;
		if(FPlatformSymbolication::LoadSymbolDatabaseForBinary(TEXT(""), Module.Name, Module.Report, Architecture, Database))
		{
			SymbolCache.Add(Database);
			Db = SymbolCache.Find(Module.Report);
		}
	}
	if((Module.Name.Len() > 0) && Db && FPlatformSymbolication::SymbolInfoForStrippedSymbol(*Db, ProgramCounter, Module.BaseOfImage, Module.Report, Info))
	{
		if(FCStringAnsi::Strlen(Info.FunctionName) > 0)
		{
			OutFunctionName = Info.FunctionName;
			ValuesSymbolised++;
		}
		if(ValuesSymbolised == 1 && FCStringAnsi::Strlen(Info.Filename) > 0)
		{
			OutFileName = Info.Filename;
			ValuesSymbolised++;
		}
		if(ValuesSymbolised == 2 && Info.LineNumber > 0)
		{
			OutLineNumber = Info.LineNumber;
			ValuesSymbolised++;
		}
	}
	
	return ValuesSymbolised;
}

static TCHAR const* FindModules(TCHAR const* CrashLog)
{
	TCHAR const* Line = FCStringWide::Strstr(CrashLog, TEXT("Binary Images:"));
	if (Line)
	{
		Line = FCStringWide::Strchr(Line, TEXT('\n'));
		check(Line);
		Line += 1;
	}
	return Line;
}

static int32 ParseModuleVersion(TCHAR const* Version, uint16& OutMajor, uint16& OutMinor, uint16& OutPatch, uint16& OutBuild)
{
	OutMajor = OutMinor = OutPatch = OutBuild = 0;
	int32 Found = swscanf(TCHAR_TO_WCHAR(Version), WTEXT("%hu.%hu.%hu"), &OutMajor, &OutMinor, &OutPatch);
	TCHAR const* CurrentStart = FCStringWide::Strchr(Version, TEXT('-'));
	if(CurrentStart)
	{
		int32 Components[3] = {0, 0, 0};
		int32 Result = swscanf(TCHAR_TO_WCHAR(CurrentStart), WTEXT("%*ls %d.%d.%d"), &Components[0], &Components[1], &Components[2]);
		OutBuild = (uint16)(Components[0] * 10000) + (Components[1] * 100) + (Components[2]);
		Found = 4;
	}
	return Found;
}

static bool ParseModuleLine(TCHAR const* ModuleLine, FCrashModuleInfo& OutModule)
{
	bool bOK = false;
	wchar_t RawModuleName[257] = {0};
	uint64 ModuleBase = 0;
	uint64 ModuleEnd = 0;
	
	int32 Found = swscanf(TCHAR_TO_WCHAR(ModuleLine), WTEXT("%lx %*ls %lx %256ls"), &ModuleBase, &ModuleEnd, RawModuleName);
	switch (Found)
	{
		case 3:
		{
			TCHAR const* VersionStart = FCStringWide::Strchr(ModuleLine, TEXT('('));
			TCHAR const* VersionEnd = FCStringWide::Strchr(ModuleLine, TEXT(')'));
			if(VersionStart && VersionEnd)
			{
				++VersionStart;
				Found += ParseModuleVersion(VersionStart, OutModule.Major, OutModule.Minor, OutModule.Patch, OutModule.Revision);
			}
			
			TCHAR const* UUIDStart = FCStringWide::Strchr(ModuleLine, TEXT('<'));
			TCHAR const* UUIDEnd = FCStringWide::Strchr(ModuleLine, TEXT('>'));
			if(UUIDStart && UUIDEnd)
			{
				++UUIDStart;
				int32 Length = FMath::Min(64, (int32)((uintptr_t)(UUIDEnd - UUIDStart)));
				OutModule.Report.Append(UUIDStart, Length);
				if(!OutModule.Report.Contains(TEXT("-")))
				{
					OutModule.Report.InsertAt(8, TEXT('-'));
					OutModule.Report.InsertAt(13, TEXT('-'));
					OutModule.Report.InsertAt(18, TEXT('-'));
					OutModule.Report.InsertAt(23, TEXT('-'));
				}
				OutModule.Report = OutModule.Report.ToUpper();
				Found++;
			}
			
			TCHAR const* Path = FCStringWide::Strchr(ModuleLine, TEXT('/'));
			if(Path)
			{
				TCHAR const* End = FCStringWide::Strchr(Path, TEXT('\r'));
				if(!End)
				{
					End = FCStringWide::Strchr(Path, TEXT('\n'));
				}
				check(End);
				
				int32 Length = FMath::Min(PATH_MAX, (int32)((uintptr_t)(End - Path)));
				OutModule.Name.Append(Path, Length);
				Found++;
				bOK = true;
			}
		}
		case 2:
		{
			OutModule.SizeOfImage = (ModuleBase - ModuleEnd);
		}
		case 1:
		{
			OutModule.BaseOfImage = ModuleBase;
			break;
		}
		default:
		{
			break;
		}
	}
	return bOK;
}

FCrashDebugHelperMac::FCrashDebugHelperMac()
{
}

FCrashDebugHelperMac::~FCrashDebugHelperMac()
{
}

bool FCrashDebugHelperMac::ParseCrashDump(const FString& InCrashDumpName, FCrashDebugInfo& OutCrashDebugInfo)
{
	SCOPED_AUTORELEASE_POOL;

	if (bInitialized == false)
	{
		UE_LOGF(LogCrashDebugHelper, Warning, "ParseCrashDump: CrashDebugHelper not initialized");
		return false;
	}
	
	FString CrashDump;
	
	NSString* CrashDumpPath = InCrashDumpName.GetNSString();
	NSError* Error = nil;
    NSData* Data = [NSData dataWithContentsOfFile: CrashDumpPath options: NSMappedRead error: &Error];
	if(Data && !Error)
	{
		PLCrashReport* CrashLog = [[PLCrashReport alloc] initWithData: Data error: &Error];
		if(CrashLog && !Error)
		{
			NSString* Report = [PLCrashReportTextFormatter stringValueForCrashReport: CrashLog withTextFormat: PLCrashReportTextFormatiOS];
			CrashDump = FString(Report);
			
			// Replace the binary PLCrashReporter file with an easily readable text dump
			FFileHelper::SaveStringToFile(CrashDump, *InCrashDumpName);
		}
		[CrashLog release];
	}
	
	if ( !CrashDump.IsEmpty() || FFileHelper::LoadFileToString( CrashDump, *InCrashDumpName ) )
	{
		// Only supports Apple crash report version 11
		int32 ReportVersion = 0;
		int32 Result = ParseReportVersion(*CrashDump, ReportVersion);
		if(Result == 1 && (ReportVersion == 11 || ReportVersion == 104))
		{
			int32 Major = 0;
			int32 Minor = 0;
			int32 Build = 0;
			int32 CLNumber = 0;
			FString Branch;
			Result = ParseVersion(*CrashDump, Major, Minor, Build, CLNumber, Branch);
			if(Result >= 3)
			{
				if (Result < 5)
				{
					OutCrashDebugInfo.EngineVersion = Build;
				}
				else
				{
					OutCrashDebugInfo.EngineVersion = CLNumber;
				}
				if(Result == 5)
				{
					OutCrashDebugInfo.SourceControlLabel = Branch;
				}
				OutCrashDebugInfo.PlatformName = TEXT("Mac");
				OutCrashDebugInfo.CrashDumpName = InCrashDumpName;
				return true;
			}
		}
	}

	return false;
}

bool FCrashDebugHelperMac::CreateMinidumpDiagnosticReport( const FString& InCrashDumpName )
{
	bool bOK = false;
	
	FString CrashDump;
	
    {
        NSString* CrashDumpPath = InCrashDumpName.GetNSString();
        NSError* Error = nil;
        NSData* Data = [NSData dataWithContentsOfFile: CrashDumpPath options: NSMappedRead error: &Error];
        if(Data && !Error)
        {
            PLCrashReport* CrashLog = [[PLCrashReport alloc] initWithData: Data error: &Error];
            if(CrashLog && !Error)
            {
                NSString* Report = [PLCrashReportTextFormatter stringValueForCrashReport: CrashLog withTextFormat: PLCrashReportTextFormatiOS];
				CrashDump = FString(Report);
				
				// Replace the binary PLCrashReporter file with an easily readable text dump
				FFileHelper::SaveStringToFile(CrashDump, *InCrashDumpName);
            }
			[CrashLog release];
        }
    }
	
	if ( !CrashDump.IsEmpty() || FFileHelper::LoadFileToString( CrashDump, *InCrashDumpName ) )
	{
		int32 ReportVersion = 0;
		int32 Result = ParseReportVersion(*CrashDump, ReportVersion);
		if(Result == 1 && (ReportVersion == 11 || ReportVersion == 104))
		{
			FString Error;
			FString ModulePath;
			FString ModuleName;
			FString FunctionName;
			FString FileName;
			FString Branch;
			FString Model;
			FString Gpu;
			
			uint64 ProgramCounter = 0;
			int32 Major = 0;
			int32 Minor = 0;
			int32 Build = 0;
			int32 CLNumber = 0;
			int32 LineNumber = 0;;
			
			Result = ParseVersion(*CrashDump, Major, Minor, Build, CLNumber, Branch);
			if(Result >= 3)
			{
				CrashInfo.EngineVersion = FEngineVersion(Major, Minor, Build, CLNumber, Branch).ToString();
			}
			
			if(Result >= 4)
			{
				CrashInfo.BuiltFromCL = CLNumber;
			}
			
			if(Result == 5 && Branch.Len() > 0)
			{
				CrashInfo.LabelName = Branch;
			}
			
			Result = ParseOS(*CrashDump, CrashInfo.SystemInfo.OSMajor, CrashInfo.SystemInfo.OSMinor, CrashInfo.SystemInfo.OSBuild, CrashInfo.SystemInfo.OSRevision);
			check(Result == 4);
			
			CrashInfo.SystemInfo.ProcessorArchitecture = PA_X64;
			
			ParseModel(*CrashDump, Model, CrashInfo.SystemInfo.ProcessorCount);
			ParseGraphics(*CrashDump, Gpu);
			CrashInfo.SystemInfo.Report = Model + Gpu;
			
			Result = ParseError(*CrashDump, CrashInfo.Exception.ExceptionString);
			check(Result == 1);
			
			Result = ParseProcessID(*CrashDump, CrashInfo.Exception.ProcessId);
			check(Result == 1);
			
			Result = ParseCrashedThread(*CrashDump, CrashInfo.Exception.ThreadId);
			check(Result == 1);
			
			Result = ParseExceptionCode(*CrashDump, CrashInfo.Exception.Code);
			check(Result == 1);

			FString Architecture;
			Result = ParseArchitecture(*CrashDump, Architecture);
			check(Result == 1);

			FCrashThreadInfo ThreadInfo;
			ThreadInfo.ThreadId = CrashInfo.Exception.ThreadId;
			ThreadInfo.SuspendCount = 0;
			
			// Parse modules now for symbolication - if we don't have the running process we need to symbolicate by UUID
			TCHAR const* ModuleLine = FindModules(*CrashDump);
			while(ModuleLine)
			{
				FCrashModuleInfo Module;
				if (ParseModuleLine(ModuleLine, Module))
				{
					CrashInfo.Modules.Push(Module);
					CrashInfo.ModuleNames.Push(FPaths::GetBaseFilename(Module.Name));
					
					ModuleLine = FCStringWide::Strchr(ModuleLine, TEXT('\n'));
					check(ModuleLine);
					ModuleLine += 1;
				}
				else
				{
					ModuleLine = nullptr;
				}
			}
			
			
			FPlatformSymbolDatabaseSet SymbolCache;
			
			bool bIsCrashLocation = true;
			TCHAR const* ThreadStackLine = FindCrashedThreadStack(*CrashDump);
			uint32 Index = 0;
			while(ThreadStackLine)
			{
				if(CrashInfo.Exception.Code == SIGTRAP)
				{
					// For ensures strip the first three lines as they are PLCrashReporter nonsense
					if(Index < 3)
					{
						ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
						if(ThreadStackLine)
						{
							ThreadStackLine += 1;
						}
						++Index;
						continue;
					}
					
					// Crash location is the 5th entry in the stack.
					bIsCrashLocation = (Index == 5);
				}
				
				Result = ParseThreadStackLine(ThreadStackLine, ModuleName, ProgramCounter, FunctionName, FileName, LineNumber);
				
				// If we got the modulename & program counter but didn't parse the filename & linenumber we can resymbolise
				if(Result > 1 && Result < 4)
				{
					// Attempt to resymbolise using CoreSymbolication
					Result += SymboliseStackInfo(SymbolCache, CrashInfo.Modules, ModuleName, Architecture, ProgramCounter, FunctionName, FileName, LineNumber);
				}
				
				// Output in our format based on the fields we actually have
				switch (Result)
				{
					case 2:
						CrashInfo.Exception.CallStackString.Push( FString::Printf( TEXT( "Unknown() Address = 0x%lx (filename not found) [in %s]" ), ProgramCounter, *ModuleName ) );
						ThreadInfo.CallStack.Push(ProgramCounter);
						
						ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
						check(ThreadStackLine);
						ThreadStackLine += 1;
						break;
						
					case 3:
					case 4:
						CrashInfo.Exception.CallStackString.Push( FString::Printf( TEXT( "%s Address = 0x%lx (filename not found) [in %s]" ), *FunctionName, ProgramCounter, *ModuleName ) );
						ThreadInfo.CallStack.Push(ProgramCounter);
						
						ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
						check(ThreadStackLine);
						ThreadStackLine += 1;
						break;
						
					case 5:
					case 6: // Function name might be parsed twice
						if(bIsCrashLocation)
						{
							if( FileName.Len() > 0 && LineNumber > 0 )
							{
								// Sync the source file where the crash occurred
								CrashInfo.SourceFile = ExtractRelativePath( TEXT( "source" ), *FileName );
								CrashInfo.SourceLineNumber = LineNumber;
								
								// Add the standard source context
								AddSourceToReport();
							}
						}
						
						CrashInfo.Exception.CallStackString.Push( FString::Printf( TEXT( "%s Address = 0x%lx [%s, line %d] [in %s]" ), *FunctionName, ProgramCounter, *FileName, LineNumber, *ModuleName ) );
						ThreadInfo.CallStack.Push(ProgramCounter);
						
						ThreadStackLine = FCStringWide::Strchr(ThreadStackLine, TEXT('\n'));
						check(ThreadStackLine);
						ThreadStackLine += 1;
						break;
						
					default:
						ThreadStackLine = nullptr;
						break;
				}
				
				++Index;
				bIsCrashLocation = false;
			}
			
			CrashInfo.Threads.Push(ThreadInfo);
			
			bOK = true;
		}
	}
	
	return bOK;
}
