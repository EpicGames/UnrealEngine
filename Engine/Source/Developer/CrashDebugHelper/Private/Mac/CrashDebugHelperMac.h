// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CrashDebugHelper.h"

class FCrashDebugHelperMac : public ICrashDebugHelper
{
public:
	FCrashDebugHelperMac();
	virtual ~FCrashDebugHelperMac();

	/**
	 *	Parse the given crash dump, determining EngineVersion of the build that produced it - if possible. 
	 *
	 *	@param	InCrashDumpName		The crash dump file to process
	 *	@param	OutCrashDebugInfo	The crash dump info extracted from the file
	 *
	 *	@return	bool				true if successful, false if not
	 */
	virtual bool ParseCrashDump(const FString& InCrashDumpName, FCrashDebugInfo& OutCrashDebugInfo) override;


	/**
	 *	Parse the given crash dump, and generate a report. 
	 *
	 *	@param	InCrashDumpName		The crash dump file to process
	 *
	 *	@return	bool				true if successful, false if not
	 */
	virtual bool CreateMinidumpDiagnosticReport( const FString& InCrashDumpName ) override;
};

typedef FCrashDebugHelperMac FCrashDebugHelper;
