// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDebugHelperWindows.h"
#include "CrashDebugHelperPrivate.h"
#include "WindowsPlatformStackWalkExt.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"

#include "Misc/EngineVersion.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <DbgHelp.h>

bool FCrashDebugHelperWindows::CreateMinidumpDiagnosticReport( const FString& InCrashDumpFilename )
{
	FWindowsPlatformStackWalkExt WindowsStackWalkExt( CrashInfo );

	const bool bReady = WindowsStackWalkExt.InitStackWalking();

	bool bResult = false;
	if( bReady && WindowsStackWalkExt.OpenDumpFile( InCrashDumpFilename ) )
	{
		if (CrashInfo.BuiltFromCL != FCrashInfo::INVALID_CHANGELIST)
		{
			// Get the build version and modules paths.
			FCrashModuleInfo ExeFileVersion;
			WindowsStackWalkExt.GetExeFileVersionAndModuleList(ExeFileVersion);

			// Init Symbols
			WindowsStackWalkExt.InitSymbols();

			// Set the symbol path based on the loaded modules
			WindowsStackWalkExt.SetSymbolPathsFromModules();

			// Get all the info we should ever need about the modules
			WindowsStackWalkExt.GetModuleInfoDetailed();

			// Get info about the system that created the minidump
			WindowsStackWalkExt.GetSystemInfo();

			// Get all the thread info
			WindowsStackWalkExt.GetThreadInfo();

			// Get exception info
			WindowsStackWalkExt.GetExceptionInfo();

			// Get the callstacks for each thread
			WindowsStackWalkExt.GetCallstacks();

			// Add the source file where the crash occurred
			AddSourceToReport();

			// Set the result
			bResult = true;
		}
		else
		{
			UE_LOGF( LogCrashDebugHelper, Warning, "Invalid built from changelist" );
		}
	}
	else
	{
		UE_LOGF( LogCrashDebugHelper, Warning, "Failed to open crash dump file: %ls", *InCrashDumpFilename );
	}

	return bResult;
}

#include "Windows/HideWindowsPlatformTypes.h"
