// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsPlatformStackWalkExt.h"
#include "CrashDebugHelper.h"
#include "CrashDebugHelperPrivate.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/MemStack.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "dbgeng.h"
#include <DbgHelp.h>
#include "Windows/HideWindowsPlatformTypes.h"

#pragma comment( lib, "dbgeng.lib" )

// Use _NT_SYMBOL_PATH for non development builds. We don't want shipping crash reporter to try to 
// access build servers for example.
#if !UE_BUILD_SHIPPING
	#define ALLOW_UNREAL_ACCESS_TO_NT_SYMBOL_PATH 1
#else
	#define ALLOW_UNREAL_ACCESS_TO_NT_SYMBOL_PATH 0
#endif

static IDebugClient5* Client = NULL;
static IDebugControl4* Control = NULL;
static IDebugSymbols3* Symbol = NULL;
static IDebugAdvanced3* Advanced = NULL;

FWindowsPlatformStackWalkExt::FWindowsPlatformStackWalkExt( FCrashInfo& InCrashInfo ) 
	: CrashInfo( InCrashInfo )
{}

FWindowsPlatformStackWalkExt::~FWindowsPlatformStackWalkExt()
{
	ShutdownStackWalking();
}

bool FWindowsPlatformStackWalkExt::InitStackWalking()
{
	if (!FWindowsPlatformMisc::CoInitialize())
	{
		return false;
	}

	check( DebugCreate( __uuidof( IDebugClient5 ), ( void** )&Client ) == S_OK );

	check( Client->QueryInterface( __uuidof( IDebugControl4 ), ( void** )&Control ) == S_OK );
	check( Client->QueryInterface( __uuidof( IDebugSymbols3 ), ( void** )&Symbol ) == S_OK );
	check( Client->QueryInterface( __uuidof( IDebugAdvanced3 ), ( void** )&Advanced ) == S_OK );

	return true;
}

void FWindowsPlatformStackWalkExt::ShutdownStackWalking()
{
	Advanced->Release();
	Symbol->Release();
	Control->Release();
	
	Client->Release();

	FWindowsPlatformMisc::CoUninitialize();
}

/**
 * 
 */
void FWindowsPlatformStackWalkExt::InitSymbols()
{
	ULONG SymOpts = 0;

	// Load line information
	SymOpts |= SYMOPT_LOAD_LINES;
	SymOpts |= SYMOPT_OMAP_FIND_NEAREST;
	// Fail if a critical error is encountered
	SymOpts |= SYMOPT_FAIL_CRITICAL_ERRORS;
	// Always load immediately; no deferred loading
	SymOpts |= SYMOPT_DEFERRED_LOADS;
	// Require an exact symbol match
	SymOpts |= SYMOPT_EXACT_SYMBOLS;
	// This option allows for undecorated names to be handled by the symbol engine.
	SymOpts |= SYMOPT_UNDNAME;

#ifdef	_DEBUG
	// Disable by default as it can be very spammy/slow.  Turn it on if you are debugging symbol look-up!
	SymOpts |= SYMOPT_DEBUG;
#endif // _DEBUG

	Symbol->SetSymbolOptions( SymOpts );
}

FString FWindowsPlatformStackWalkExt::ExtractRelativePath( const TCHAR* BaseName, TCHAR* FullName )
{
	FString FullPath = FString( FullName ).ToLower();
	FullPath.ReplaceInline( TEXT( "\\" ), TEXT( "/" ) );

	TArray<FString> Components;
	int32 Count = FullPath.ParseIntoArray( Components, TEXT( "/" ), true );
	FullPath = TEXT( "" );

	for( int32 Index = 0; Index < Count; Index++ )
	{
		if( Components[Index] == BaseName )
		{
			if( Index > 0 )
			{
				for( int32 Inner = Index - 1; Inner < Count; Inner++ )
				{
					FullPath += Components[Inner];
					if( Inner < Count - 1 )
					{
						FullPath += TEXT( "/" );
					}
				}
			}
			break;
		}
	}

	return FullPath;
}

void FWindowsPlatformStackWalkExt::GetExeFileVersionAndModuleList( FCrashModuleInfo& out_ExeFileVersion )
{
	// The the number of loaded modules
	ULONG LoadedModuleCount = 0;
	ULONG UnloadedModuleCount = 0;
	Symbol->GetNumberModules( &LoadedModuleCount, &UnloadedModuleCount );

	UE_LOGF( LogCrashDebugHelper, Log, "Modules loaded: %i, unloaded: %i", LoadedModuleCount, UnloadedModuleCount );

	// Find the relative names of all the modules so we know which files to sync
	int ExecutableIndex = -1;
	for( uint32 ModuleIndex = 0; ModuleIndex < LoadedModuleCount; ModuleIndex++ )
	{
		ULONG64 ModuleBase = 0;
		Symbol->GetModuleByIndex( ModuleIndex, &ModuleBase );

		// Get the full path of the module name
		TCHAR ModuleName[MAX_PATH] = {0};
		Symbol->GetModuleNameStringWide( DEBUG_MODNAME_IMAGE, ModuleIndex, ModuleBase, ModuleName, MAX_PATH, NULL );

		const FString RelativeModuleName = ExtractRelativePath( TEXT( "binaries" ), ModuleName );
		// Get the exe, which we extract the version number, so we know what label to sync to
		if (RelativeModuleName.Len() > 0 && RelativeModuleName.EndsWith( TEXT( ".exe" ) ))
		{
			ExecutableIndex = ModuleIndex;
		}

		// Add only modules in Binaries folders
		if (RelativeModuleName.Len() > 0)
		{
			CrashInfo.ModuleNames.Add( ModuleName );
		}
		else
		{
			SystemModuleNames.Add(ModuleName);
		}
	}

	// Get the executable version info.
	if( ExecutableIndex > -1 )
	{
		VS_FIXEDFILEINFO VersionInfo = {0};
		Symbol->GetModuleVersionInformationWide( ExecutableIndex, 0, TEXT( "\\" ), &VersionInfo, sizeof( VS_FIXEDFILEINFO ), NULL );

		out_ExeFileVersion.Major = HIWORD( VersionInfo.dwProductVersionMS );
		out_ExeFileVersion.Minor = LOWORD( VersionInfo.dwProductVersionMS );
		out_ExeFileVersion.Patch = HIWORD( VersionInfo.dwProductVersionLS );
	}
	else
	{
		UE_LOGF( LogCrashDebugHelper, Warning, "Unable to locate the executable" );
	}
}

void FWindowsPlatformStackWalkExt::SetSymbolPathsFromModules()
{
	FString CombinedPath = TEXT( "" );

	{
		// Use symbol cache from command line
		FString DebugSymbols;
		if (FParse::Value(FCommandLine::Get(), TEXT("DebugSymbols="), DebugSymbols))
		{
			CombinedPath += TEXT("SRV*");
			CombinedPath += DebugSymbols;
			CombinedPath += TEXT(";");
		}

		// Iterate over all loaded modules.
		TSet<FString> SymbolPaths;
		for (const FString& Filename : CrashInfo.ModuleNames)
		{
			const FString Path = FPaths::GetPath(Filename);
			if (Path.Len() > 0)
			{
				SymbolPaths.Add(Path);
			}
		}

		for (const FString& SymbolPath : SymbolPaths)
		{
			CombinedPath += SymbolPath;
			CombinedPath += TEXT(";");
		}

#if ALLOW_UNREAL_ACCESS_TO_NT_SYMBOL_PATH
		FString SymbolPathEnvironmentVariable = FPlatformMisc::GetEnvironmentVariable(L"_NT_SYMBOL_PATH");
		if (!SymbolPathEnvironmentVariable.IsEmpty())
		{
			CombinedPath += SymbolPathEnvironmentVariable;
			CombinedPath += ";";
		}
#endif
	}

	TSet<FString> ImagePathSet;
	FString ImagePathnames(CombinedPath);
	for (const FString& SysImagePathname : SystemModuleNames)
	{
		FString Path = FPaths::GetPath(SysImagePathname);
		if (Path.Len() > 0)
		{
			ImagePathSet.Add(Path);
		}
	}

	for (const FString& SysImgPath : ImagePathSet)
	{
		ImagePathnames += SysImgPath;
		ImagePathnames += TEXT(";");
	}

	// Set the symbol path
	Symbol->SetImagePathWide( *ImagePathnames );
	Symbol->SetSymbolPathWide( *CombinedPath );

	// Add in syncing of the Microsoft symbol servers if requested
	if( FParse::Param( FCommandLine::Get(), TEXT( "SyncMicrosoftSymbols" ) ) )
	{
		FString BinariesDir = FString(FPlatformProcess::BaseDir());
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		BinariesDir = FPaths::Combine(BinariesDir, TEXT("arm64"));
#endif
		if ( !FPaths::FileExists( FPaths::Combine(*BinariesDir, TEXT("symsrv.dll")) ) )
		{
			UE_LOGF( LogCrashDebugHelper, Warning, "Error: symsrv.dll was not detected in: %ls. Microsoft symbols will not be downloaded!", *BinariesDir );
		}
		if ( !FPaths::FileExists( FPaths::Combine(*BinariesDir, TEXT("symsrv.yes")) ) )
		{
			UE_LOGF( LogCrashDebugHelper, Warning, "symsrv.yes was not detected in: %ls. This will cause a popup to confirm the license.", *BinariesDir );
		}
		if ( !FPaths::FileExists( FPaths::Combine(*BinariesDir, TEXT("dbghelp.dll")) ) )
		{
			UE_LOGF( LogCrashDebugHelper, Warning, "Error: dbghelp.dll was not detected in: %ls. Microsoft symbols will not be downloaded!", *BinariesDir );
		}

		Symbol->AppendImagePathWide( TEXT( "SRV*..\\..\\Intermediate\\SymbolCache*http://msdl.microsoft.com/download/symbols" ) );
		Symbol->AppendSymbolPathWide( TEXT( "SRV*..\\..\\Intermediate\\SymbolCache*http://msdl.microsoft.com/download/symbols" ) );
	}

	TCHAR SymbolPath[16384 * 2] = { 0 }; // On large projects, we usually need more than 16K because the list of module is long.

	Symbol->GetSymbolPathWide( SymbolPath, UE_ARRAY_COUNT(SymbolPath), NULL );
	TArray<FString> SymbolPaths;
	FString( SymbolPath ).ParseIntoArray(SymbolPaths, TEXT(";"), true );

	UE_LOGF( LogCrashDebugHelper, Log, "Symbol paths" );
	for( const auto& It : SymbolPaths )
	{
		UE_LOGF( LogCrashDebugHelper, Log, "    %ls", *It );
	}

	Symbol->GetImagePathWide( SymbolPath, UE_ARRAY_COUNT( SymbolPath ), NULL );
	TArray<FString> ImagePaths;
	FString( SymbolPath ).ParseIntoArray( ImagePaths, TEXT( ";" ), true );
	
	UE_LOGF( LogCrashDebugHelper, Log, "Image paths" );
	for( const auto& It : ImagePaths )
	{
		UE_LOGF( LogCrashDebugHelper, Log, "    %ls", *It );
	}
}

/** Helper class used to sort modules by name. */
class FSortModulesByName
{
public:
	bool operator()( const FCrashModuleInfo& A, const FCrashModuleInfo& B ) const
	{
		if( A.Extension == TEXT( ".exe" ) )
		{
			return true;
		}

		if( A.Extension != B.Extension )
		{
			// Sort any exe modules to the top
			return ( A.Extension > B.Extension );
		}

		// alphabetise all the dlls
		return ( A.Name < B.Name );
	}
};

void FWindowsPlatformStackWalkExt::GetModuleInfoDetailed()
{
	// The the number of loaded modules
	ULONG LoadedModuleCount = 0;
	ULONG UnloadedModuleCount = 0;
	Symbol->GetNumberModules( &LoadedModuleCount, &UnloadedModuleCount );

	CrashInfo.Modules.Empty( LoadedModuleCount );
	for( uint32 ModuleIndex = 0; ModuleIndex < LoadedModuleCount; ModuleIndex++ )
	{
		FCrashModuleInfo* CrashModule = new ( CrashInfo.Modules ) FCrashModuleInfo();

		ULONG64 ModuleBase = 0;
		Symbol->GetModuleByIndex( ModuleIndex, &ModuleBase );

		// Get the full path of the module name
		TCHAR ModuleName[MAX_PATH] = { 0 };
		Symbol->GetModuleNameStringWide( DEBUG_MODNAME_IMAGE, ModuleIndex, ModuleBase, ModuleName, MAX_PATH, NULL );

		FString RelativeModuleName = ExtractRelativePath( TEXT( "binaries" ), ModuleName );
		if( RelativeModuleName.Len() > 0 )
		{
			CrashModule->Name = RelativeModuleName;
		}
		else
		{
			CrashModule->Name = ModuleName;
		}
		CrashModule->Extension = CrashModule->Name.Right( 4 ).ToLower();
		CrashModule->BaseOfImage = ModuleBase;

		DEBUG_MODULE_PARAMETERS ModuleParameters = { 0 };
		Symbol->GetModuleParameters( 1, NULL, ModuleIndex, &ModuleParameters ); 
		CrashModule->SizeOfImage = ModuleParameters.Size;

		VS_FIXEDFILEINFO VersionInfo = { 0 };
		Symbol->GetModuleVersionInformationWide( ModuleIndex, ModuleBase, TEXT( "\\" ), &VersionInfo, sizeof( VS_FIXEDFILEINFO ), NULL );

		CrashModule->Major = HIWORD( VersionInfo.dwProductVersionMS );
		CrashModule->Minor = LOWORD( VersionInfo.dwProductVersionMS );
		CrashModule->Patch = HIWORD( VersionInfo.dwProductVersionLS );
		CrashModule->Revision = LOWORD( VersionInfo.dwProductVersionLS );

		// NOTE: The function below was commented because in some occasions (unknown reason), it triggered a search for modules on the user disk,
		//       scanning all projects assets for each module, taking an unacceptable long time in very large projects. This could be observed with
		//       SysInternal ProcessMonitor. The original intent for calling ReloadWide() is unknown, but normally, this function is meant
		//       to control when the modules are reloaded, for example if the .pdb were about to be deleted, we would want to reload and cache the
		//       symbols before losing them. Since we configured the debug engine with SYMOPT_DEFERRED_LOADS to only load symbols if needed,
		//       reloading them all here looks counter-productive. Commenting it did not show any obvious side effects other than fixing the
		//       very long module search time issue.
		// Ensure all the images are synced - need the full path here
		//Symbol->ReloadWide( *CrashModule->Name );
	}

	CrashInfo.Modules.Sort( FSortModulesByName() );
}

bool FWindowsPlatformStackWalkExt::IsOffsetWithinModules( uint64 Offset )
{
	for( int ModuleIndex = 0; ModuleIndex < CrashInfo.Modules.Num(); ModuleIndex++ )
	{
		FCrashModuleInfo& CrashModule = CrashInfo.Modules[ModuleIndex];
		if( Offset >= CrashModule.BaseOfImage && Offset < CrashModule.BaseOfImage + CrashModule.SizeOfImage )
		{
			return true;
		}
	}

	return false;
}

void FWindowsPlatformStackWalkExt::GetSystemInfo()
{
	FCrashSystemInfo& SystemInfo = CrashInfo.SystemInfo;

	ULONG PlatformId = 0;
	ULONG Major = 0;
	ULONG Minor = 0;
	ULONG Build = 0;
	ULONG Revision = 0;
	Control->GetSystemVersionValues( &PlatformId, &Major, &Minor, &Build, &Revision );

	SystemInfo.OSMajor = ( uint16 )Major;
	SystemInfo.OSMinor = ( uint16 )Minor;
	SystemInfo.OSBuild = ( uint16 )Build;
	SystemInfo.OSRevision = ( uint16 )Revision;

	ULONG ProcessorType = 0;
	Control->GetActualProcessorType( &ProcessorType );

	switch( ProcessorType )
	{
	case IMAGE_FILE_MACHINE_I386:
		// x86
		CrashInfo.SystemInfo.ProcessorArchitecture = PA_X86;
		break;

	case IMAGE_FILE_MACHINE_ARM:
		// ARM
		CrashInfo.SystemInfo.ProcessorArchitecture = PA_ARM;
		break;

	case IMAGE_FILE_MACHINE_AMD64:
		// x64
		CrashInfo.SystemInfo.ProcessorArchitecture = PA_X64;
		break;

	default: 
		break;
	};

	ULONG ProcessorCount = 0;
	Control->GetNumberProcessors( &ProcessorCount );
	SystemInfo.ProcessorCount = ProcessorCount;
}

void FWindowsPlatformStackWalkExt::GetExceptionInfo()
{
	FCrashExceptionInfo& Exception = CrashInfo.Exception;

	ULONG ExceptionType = 0;
	ULONG ProcessID = 0;
	ULONG ThreadId = 0;
	TCHAR Description[MAX_PATH] = { 0 };
	Control->GetLastEventInformationWide( &ExceptionType, &ProcessID, &ThreadId, NULL, 0, NULL, Description, UE_ARRAY_COUNT( Description ), NULL );

	Exception.Code = ExceptionType;
	Exception.ProcessId = ProcessID;
	Exception.ThreadId = ThreadId;
	Exception.ExceptionString = Description;
}


void FWindowsPlatformStackWalkExt::GetCallstacks()
{
	const int32 MAX_NAME_LENGTH = FProgramCounterSymbolInfo::MAX_NAME_LENGTH;

	FCrashExceptionInfo& Exception = CrashInfo.Exception;

	FMemMark Mark(FMemStack::Get());

	//const float int int32 FString
	const int32 ContextSize = 4096;
	byte* Context = new(FMemStack::Get()) byte[ContextSize];
	ULONG DebugEvent = 0;
	ULONG ProcessID = 0;
	ULONG ThreadID = 0;
	ULONG ContextUsed = 0;

	// Get the context of the crashed thread
	HRESULT hr = Control->GetStoredEventInformation(&DebugEvent, &ProcessID, &ThreadID, Context, ContextSize, &ContextUsed, NULL, 0, 0);
	if( FAILED(hr) )
	{
		return;
	}

	// Some magic number checks
	if( ContextUsed == 716 )
	{
		UE_LOGF( LogCrashDebugHelper, Log, "Context size matches x86 sizeof( CONTEXT )" );
	}
	else if( ContextUsed == 1232 )
	{
		UE_LOGF( LogCrashDebugHelper, Log, "Context size matches x64 sizeof( CONTEXT )" );
	}

	// Get the entire stack trace
	const uint32 MaxFrames = 8192;
	const uint32 MaxFramesSize = MaxFrames * ContextUsed;

	DEBUG_STACK_FRAME* StackFrames = new(FMemStack::Get()) DEBUG_STACK_FRAME[MaxFrames];
	ULONG Count = 0;
	bool bFoundSourceFile = false;
	void* ContextData = FMemStack::Get().PushBytes( MaxFramesSize, 0 );
	FMemory::Memzero( ContextData, MaxFramesSize );
	UE_LOGF(LogCrashDebugHelper, Log, "Running GetContextStackTrace()");
	HRESULT HR = Control->GetContextStackTrace( Context, ContextUsed, StackFrames, MaxFrames, ContextData, MaxFramesSize, ContextUsed, &Count );
	UE_LOGF(LogCrashDebugHelper, Log, "GetContextStackTrace() got %d frames", Count);

	int32 AssertOrEnsureIndex = -1;

	for( uint32 StackIndex = 0; StackIndex < Count; StackIndex++ )
	{	
		const uint64 Offset = StackFrames[StackIndex].InstructionOffset;

		if( IsOffsetWithinModules( Offset ) )
		{
			// Get the module, function, and offset
			uint64 Displacement = 0;
			TCHAR NameByOffset[MAX_PATH] = {0};
			Symbol->GetNameByOffsetWide( Offset, NameByOffset, ARRAYSIZE( NameByOffset ) - 1, NULL, &Displacement );
			FString ModuleAndFunction = NameByOffset;

			// Don't care about any more entries higher than this
			if (ModuleAndFunction.Contains( TEXT( "tmainCRTStartup" ) ) || ModuleAndFunction.Contains( TEXT( "FRunnableThreadWin::GuardedRun" ) ))
			{
				break;
			}

			// Look for source file name and line number
			TCHAR SourceName[MAX_PATH] = { 0 };
			ULONG LineNumber = 0;
			Symbol->GetLineByOffsetWide( Offset, &LineNumber, SourceName, ARRAYSIZE( SourceName ) - 1, NULL, NULL );

			// Remember the top of the stack to locate in the source file
			if( !bFoundSourceFile && FCString::Strlen( SourceName ) > 0 && LineNumber > 0 )
			{
				CrashInfo.SourceFile = ExtractRelativePath( TEXT( "source" ), SourceName );
				CrashInfo.SourceLineNumber = LineNumber;
				bFoundSourceFile = true;
			}

			FString ModuleName;
			FString FunctionName;
			// According to MSDN, the symbol name will include an ! if the function name could be discovered, delimiting it from the module name
			// https://msdn.microsoft.com/en-us/library/windows/hardware/ff547186(v=vs.85).aspx
			if( ModuleAndFunction.Contains( TEXT( "!" ) ) )
			{
				ModuleAndFunction.Split( TEXT( "!" ), &ModuleName, &FunctionName );
				FunctionName += TEXT( "()" );
			}
			else
			{
				ModuleName = ModuleAndFunction;
			}

			// FString InModuleName, FString InFunctionName, FString InFilename, uint32 InLineNumber, uint64 InSymbolDisplacement, uint64 InOffsetInModule, uint64 InProgramCounter
			FProgramCounterSymbolInfoEx SymbolInfo( ModuleName, FunctionName, SourceName, LineNumber, Displacement, Offset, 0 );
			FString GenericFormattedCallstackLine;
			FGenericPlatformStackWalk::SymbolInfoToHumanReadableStringEx( SymbolInfo, GenericFormattedCallstackLine );
			Exception.CallStackString.Add( GenericFormattedCallstackLine );

			UE_LOGF( LogCrashDebugHelper, Log, "%3u: %ls", StackIndex, *GenericFormattedCallstackLine );
		}
	}
}


bool FWindowsPlatformStackWalkExt::OpenDumpFile( const FString& InCrashDumpFilename )
{
	const bool bFound = IFileManager::Get().FileSize( *InCrashDumpFilename ) != INDEX_NONE;
	if( !bFound )
	{
		UE_LOGF( LogCrashDebugHelper, Warning, "Failed to find minidump file: %ls", *InCrashDumpFilename );
		return false;
	}

	HRESULT hr = Client->OpenDumpFileWide(*InCrashDumpFilename, NULL);
	if(FAILED(hr) )
	{
		UE_LOGF( LogCrashDebugHelper, Warning, "Failed to open minidump file: %ls", *InCrashDumpFilename );
		return false;
	}

	if( Control->WaitForEvent( 0, INFINITE ) != S_OK )
	{
		UE_LOGF( LogCrashDebugHelper, Warning, "Failed while waiting for minidump to load: %ls", *InCrashDumpFilename );
		return false;
	}

	UE_LOGF( LogCrashDebugHelper, Log, "Successfully opened minidump: %ls", *InCrashDumpFilename );
	return true;
}
