// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FCrashInfo;
class FCrashModuleInfo;

/**
 *	Windows implementation of stack walking using the COM interface IDebugClient5. 
 */
struct FWindowsPlatformStackWalkExt
{
	/** Default constructor. */
	FWindowsPlatformStackWalkExt( FCrashInfo& InCrashInfo );

	/** Destructor. */
	~FWindowsPlatformStackWalkExt();

	/** Initializes the COM interface to grab stacks. */
	bool InitStackWalking();

	/** Shutdowns COM. */
	void ShutdownStackWalking();

	/** Sets the options we want for symbol lookup. */
	void InitSymbols();

	/** Grabs the branch relative path of the binary. */
	static FString ExtractRelativePath( const TCHAR* BaseName, TCHAR* FullName );

	/** Gets the exe file versions and lists all modules. */
	void GetExeFileVersionAndModuleList( FCrashModuleInfo& out_ExeFileVersion );

	/** Set the symbol paths based on the module paths. */
	void SetSymbolPathsFromModules();

	/** Gets detailed info about each module. */
	void GetModuleInfoDetailed();

	/** Check to see if the stack address resides within one of the loaded modules i.e. is it valid?. */
	bool IsOffsetWithinModules( uint64 Offset );

	/** Extract the system info of the crash from the minidump. */
	void GetSystemInfo();

	/** Extracts the thread info from the minidump. */
	void GetThreadInfo(){}

	/** Extracts info about the exception that caused the crash. */
	void GetExceptionInfo();

	/** Gets the callstack of the crash. */
	void GetCallstacks();

	/** Opens a minidump as a new session. */
	bool OpenDumpFile( const FString& DumpFileName );

protected:
	/** Reference to the crash info. */
	FCrashInfo& CrashInfo;

	/** List of modules that are not from the engine. */
	TArray<FString> SystemModuleNames;
};
