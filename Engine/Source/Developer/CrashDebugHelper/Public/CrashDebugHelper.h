// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FArchive;

enum EProcessorArchitecture
{
	PA_UNKNOWN,
	PA_ARM,
	PA_X86,
	PA_X64,
};

/** 
 * Details of a module from a crash dump
 */
class FCrashModuleInfo
{
public:
	FString Report;

	FString Name;
	FString Extension;
	uint64 BaseOfImage = 0;
	uint32 SizeOfImage = 0;
	uint16 Major = 0;
	uint16 Minor = 0;
	uint16 Patch = 0;
	uint16 Revision = 0;
};

/** 
 * Details about a thread from a crash dump
 */
class FCrashThreadInfo
{
public:
	FString Report;
	FString ThreadName;

	uint32 ThreadId = 0;
	uint32 SuspendCount = 0;
	bool IsCrashing = false;

	TArray<uint64> CallStack;
	FString PortableCallStackString;
};

/** 
 * Details about the exception in the crash dump
 */
class FCrashExceptionInfo
{
public:
	FString Report;

	uint32 ProcessId = 0;
	uint32 ThreadId = 0;
	uint32 Code = 0;
	FString ExceptionString;

	TArray<FString> CallStackString;
	TArray<FString> PortableCallStackString;
};

/** 
 * Details about the system the crash dump occurred on
 */
class FCrashSystemInfo
{
public:
	FString Report;

	EProcessorArchitecture ProcessorArchitecture = PA_UNKNOWN;
	uint32 ProcessorCount = 0;

	uint16 OSMajor = 0;
	uint16 OSMinor = 0;
	uint16 OSBuild = 0;
	uint16 OSRevision = 0;
};

// #TODO 2015-07-24 Refactor
/** A platform independent representation of a crash */
class CRASHDEBUGHELPER_API FCrashInfo
{
public:
	enum
	{
		/** An invalid changelist, something went wrong. */
		INVALID_CHANGELIST = -1,
	};

	/** Report log. */
	FString Report;

	/** The depot name, indicate where the executables and symbols are stored. */
	FString DepotName;

	/** Product version, based on FEngineVersion. */
	FString EngineVersion;

	/** Build version string. */
	FString BuildVersion;

	/** CL built from. */
	int32 BuiltFromCL = INVALID_CHANGELIST;

	/** The label the describes the executables and symbols. */
	FString LabelName;

	/** The network path where the executables are stored. */
	FString ExecutablesPath;

	/** The network path where the symbols are stored. */
	FString SymbolsPath;

	FString SourceFile;
	uint32 SourceLineNumber = 0;
	TArray<FString> SourceContext;

	/** Only modules names, retrieved from the minidump file. */
	TArray<FString> ModuleNames;

	FCrashSystemInfo SystemInfo;
	FCrashExceptionInfo Exception;
	TArray<FCrashThreadInfo> Threads;
	TArray<FCrashModuleInfo> Modules;

	FString PlatformName;
	FString PlatformVariantName;

	/** 
	 * Generate a report for the crash in the requested path
	 */
	void GenerateReport( const FString& DiagnosticsPath );

	/** 
	 * Handle logging
	 */
	void Log( FString Line );

private:
	/** 
	 * Convert the processor architecture to a human readable string
	 */
	static const TCHAR* GetProcessorArchitecture( EProcessorArchitecture PA );

	/** 
	* Write a line as UTF-8 to a file
	*/
	static void WriteLine(FArchive* ReportFile, const TCHAR* Line = nullptr);
	static void WriteLine(FArchive* ReportFile, const FString& Line);
};

/** Helper structure for tracking crash debug information */
struct FCrashDebugInfo
{
	/** The name of the crash dump file */
	FString CrashDumpName;
	/** The engine version of the crash dump build */
	int32 EngineVersion;
	/** The platform of the crash dump build */
	FString PlatformName;
	/** The source control label of the crash dump build */
	FString SourceControlLabel;
};

/** The public interface for the crash dump handler singleton. */
class CRASHDEBUGHELPER_API ICrashDebugHelper
{
public:
	/** Virtual destructor */
	virtual ~ICrashDebugHelper() = default;

	/**
	 *	Initialize the helper
	 *
	 *	@return	bool		true if successful, false if not
	 */
	virtual bool Init();

	/**
	 *	Parse the given crash dump, determining EngineVersion of the build that produced it - if possible. 
	 *
	 *	@param	InCrashDumpName		The crash dump file to process
	 *	@param	OutCrashDebugInfo	The crash dump info extracted from the file
	 *
	 *	@return	bool				true if successful, false if not
	 *	
	 *	Only used by Mac, to be removed.
	 */
	virtual bool ParseCrashDump(const FString& InCrashDumpName, FCrashDebugInfo& OutCrashDebugInfo)
	{
		return false;
	}

	/**
	 *	Parse the given crash dump, and generate a report. 
	 *
	 *	@param	InCrashDumpName		The crash dump file to process
	 *
	 *	@return	bool				true if successful, false if not
	 */
	virtual bool CreateMinidumpDiagnosticReport( const FString& InCrashDumpName )
	{
		return false;
	}

	/**
	 *	Extract lines from a source file, and add to the crash report.
	 */
	virtual void AddSourceToReport();

protected:
	/**
	 *	Load the given ANSI text file to an array of strings - one FString per line of the file.
	 *	Intended for use in simple text parsing actions
	 *
	 *	@param	OutStrings			The array of FStrings to fill in
	 *
	 *	@return	bool				true if successful, false if not
	 */
	bool ReadSourceFile( TArray<FString>& OutStrings );

public:
	/** A platform independent representation of a crash */
	FCrashInfo CrashInfo;

protected:
	/** Indicates that the crash handler is ready to do work */
	bool bInitialized = false;
};
