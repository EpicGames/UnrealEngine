// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class ICrashDebugHelper;

class FCrashDebugHelperModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	/** Gets the crash debug helper singleton or returns NULL */
	CRASHDEBUGHELPER_API ICrashDebugHelper* Get();
	
	/** Gets a new crash debug helper singleton for use with multiple report sending */
	CRASHDEBUGHELPER_API ICrashDebugHelper* GetNew();

private:
	class ICrashDebugHelper* CrashDebugHelper;
};
